[Plugins]
;+ProgramEnabledPlugins="AddYourPluginNameHere"
